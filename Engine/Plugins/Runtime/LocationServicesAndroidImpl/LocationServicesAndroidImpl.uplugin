{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mobile Location Services - Android Implementation",
	"Description": "Android implementation for blueprint access for location data from mobile devices",
	"Category": "Mobile",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LocationServicesAndroidEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LocationServicesAndroidImpl",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Android"
			]
		}
	],
	"Plugins": [
		{
			"Name": "LocationServicesBPLibrary",
			"Enabled": true
		},
		{
			"Name": "AndroidPermission",
			"Enabled": true
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"IsExperimentalVersion": false
}