// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "LocationServicesAndroidSettings.generated.h"

UCLASS(config = Engine, defaultconfig)
class ULocationServicesAndroidSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(config, EditAnywhere, Category=LocationServices, meta=(DisplayName = "Enable Coarse Location Accuracy (Network Provider)"))
	bool bCoarseLocationEnabled;

	UPROPERTY(config, EditAnywhere, Category= LocationServices, meta=(DisplayName = "Enable Fine Location Accuracy (GPS Provider)"))
	bool bFineLocationEnabled;

	UPROPERTY(config, EditAnywhere, Category= LocationServices, meta=(DisplayName = "Enable Location Updates"))
	bool bLocationUpdatesEnabled;
};
