// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LocationServicesAndroidEditor : ModuleRules
{
    public LocationServicesAndroidEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"AndroidPermission"
				}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{
				"Core",
				"CoreUObject",
                "Engine",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] 
				{
					"Settings",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] 
				{
					"Settings",
				}
			);
		}
	}
}
