// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesAndroidEditor.h"
#include "LocationServicesAndroidSettings.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#endif


IMPLEMENT_MODULE( FLocationServicesAndroidEditorModule, LocationServicesAndroidEditor );

#define LOCTEXT_NAMESPACE "FLocationServicesAndroidEditorModule"

void FLocationServicesAndroidEditorModule::StartupModule()
{
#if WITH_EDITOR
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "Location Services Android",
			LOCTEXT("LocationServicesAndroidSettingsName", "Location Services - Android"),
			LOCTEXT("LocationServicesAndroidSettingsDescription", "Configure the Location Services settings for Android"),
			GetMutableDefault<ULocationServicesAndroidSettings>()
		);
	}
#endif // WITH_EDITOR
}

void FLocationServicesAndroidEditorModule::ShutdownModule()
{
#if WITH_EDITOR
	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Location Services Android");
	}
#endif
}

ULocationServicesAndroidSettings::ULocationServicesAndroidSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCoarseLocationEnabled = true;
	bFineLocationEnabled = true;
	bLocationUpdatesEnabled = true;
}

#undef LOCTEXT_NAMESPACE
