// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class LocationServicesAndroidImpl : ModuleRules
	{
        public LocationServicesAndroidImpl(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePathModuleNames.AddRange
			(
				new string[]
				{
					"LocationServicesBPLibrary",
					"LocationServicesAndroidImpl", 
					"AndroidPermission"
				}
			);
					
			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Launch",
					"LocationServicesBPLibrary",
					// ... add private dependencies that you statically link with here ...
				}
			);
			
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"LocationServicesBPLibrary",
					// ... add private dependencies that you statically link with here ...
				}
			);

			
			
			// Additional Frameworks and Libraries for Android
			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "LocationServicesAndroidImpl_UPL.xml"));
			}
        }
	}
}
