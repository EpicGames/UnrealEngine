<?xml version="1.0" encoding="utf-8"?>
<!--OnlineSubsystemGameCircle plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="LocationServicesAndroidImpl init"/>
		<setBoolFromProperty result="bCoarseLocationEnabled" ini="Engine" section="/Script/LocationServicesAndroidEditor.LocationServicesAndroidSettings" property="bCoarseLocationEnabled" default="true"/>
		<setBoolFromProperty result="bFineLocationEnabled" ini="Engine" section="/Script/LocationServicesAndroidEditor.LocationServicesAndroidSettings" property="bFineLocationEnabled" default="true"/>
		<setBoolFromProperty result="bLocationUpdatesEnabled" ini="Engine" section="/Script/LocationServicesAndroidEditor.LocationServicesAndroidSettings" property="bLocationUpdatesEnabled" default="true"/>
	</init>

  <!-- updates applied to AndroidManifest.xml -->
  <androidManifestUpdates>
  
	<if condition="bCoarseLocationEnabled">
		<true>
			<addPermission android:name="android.permission.ACCESS_COARSE_LOCATION"/>
			<addFeature android:name="android.hardware.location.network"/>
		</true>
	</if>
	<if condition="bFineLocationEnabled">
		<true>
			<addPermission android:name="android.permission.ACCESS_FINE_LOCATION"/>
			<addFeature android:name="android.hardware.location.gps"/>
		</true>
	</if>
	<if condition="bLocationUpdatesEnabled">
		<true>
			<addPermission android:name="android.permission.CONTROL_LOCATION_UPDATES"/>
		</true>
	</if>

  </androidManifestUpdates>
  
  <!-- GameActivity additions -->
  <gameActivityImportAdditions>
	<insert>
	// Needed for Location Services
	import android.location.Location;
	import android.location.LocationListener;
	import android.location.LocationManager;
	import android.location.LocationProvider;
	import android.location.Criteria;
	</insert>
  </gameActivityImportAdditions>
	
  <gameActivityClassAdditions>
	<insert>

    private int LocationServicesAccuracy;
	private int LocationServicesPower;
    private float LocationServicesUpdateFrequency;
    private float LocationServicesMinDistance;
    private Location LastReportedLocation = null;
    private LocationListener LocationListen	= null;

    public boolean AndroidThunkJava_InitLocationServices(int Accuracy, float UpdateFrequency, float MinDistanceFilter)
    {
		Log.debug("AndroidThunkJava_InitLocationServices");
      
		//translate the provided Accuracy into Criteria and Distance values
		switch(Accuracy)
		{
			case 0: //LA_ThreeKilometers: 
			{
				LocationServicesAccuracy = Criteria.ACCURACY_LOW;
				LocationServicesPower = Criteria.POWER_LOW;
				break;
			}
			case 1: //LA_OneKilometer: 
			{
				LocationServicesAccuracy = Criteria.ACCURACY_LOW;
				LocationServicesPower = Criteria.POWER_MEDIUM;
				break;
			}
			case 2: //LA_HundredMeters: 
			{
				LocationServicesAccuracy = Criteria.ACCURACY_MEDIUM;
				LocationServicesPower = Criteria.POWER_MEDIUM;
				break;
			}
			case 3: //LA_TenMeters: 
			{
				LocationServicesAccuracy = Criteria.ACCURACY_HIGH;
				LocationServicesPower = Criteria.POWER_MEDIUM;
				break;
			}
			case 4: //LA_Best: 
			case 5: //LA_Navigation:
			{
				LocationServicesAccuracy = Criteria.ACCURACY_HIGH;
				LocationServicesPower = Criteria.POWER_HIGH;
				break;
			}
			default: 
			{
				LocationServicesAccuracy = Criteria.ACCURACY_LOW;
				LocationServicesPower = Criteria.POWER_LOW;
				Log.debug("AndroidThunkJava_InitLocationServices - init with invalid Accuracy value:  " + Accuracy + ", setting lowest defaults\n");
				break;
			}
		}

		LocationServicesMinDistance = MinDistanceFilter;
		LocationServicesUpdateFrequency = UpdateFrequency;

		return true;
    }

    public boolean AndroidThunkJava_StartLocationService()
    {
		Log.debug("AndroidThunkJava_StartLocationService");

		LocationManager locationManager = (LocationManager) this.getSystemService(Context.LOCATION_SERVICE);

		if(locationManager != null)
		{
			LocationListen = new LocationListener()
			{
				public void onLocationChanged(Location location)
				{		  
					// Called when a new location is found by the network location provider. Save off the value for GetLastKnownLocation and send off an event
					LastReportedLocation = location;
			  
					nativeHandleLocationChanged(LastReportedLocation.getTime(),
						LastReportedLocation.getLongitude(),
						LastReportedLocation.getLatitude(),
						LastReportedLocation.getAccuracy(),
						LastReportedLocation.getAltitude());
				}

				public void onStatusChanged(String provider, int status, Bundle extras)
				{
					switch(status)
					{
						case LocationProvider.AVAILABLE:
							Log.debug("onStatusChanged - " + provider + " available again\n");
							break;
						case LocationProvider.OUT_OF_SERVICE:
							Log.debug("onStatusChanged - " + provider + "  out of service\n");
							break;
						case LocationProvider.TEMPORARILY_UNAVAILABLE:
							Log.debug("onStatusChanged - " + provider + "  temporarily unavailable\n");
							break;
					}
				}
        
				public void onProviderEnabled(String provider)
				{
					Log.debug("onProviderEnabled: " + provider);
				}

				public void onProviderDisabled(String provider)
				{
					Log.debug("onProviderDisabled " + provider);
				}
			};
		  
			//requestLocationUpdates must run on main thread
			activityContext.runOnUiThread(new Runnable()
			{
				@Override
				public void run()
				{
					try
					{
						LocationManager locationManager = (LocationManager) activityContext.getSystemService(Context.LOCATION_SERVICE);

						String providerName = GetBestLocationProvider();

						locationManager.requestLocationUpdates(providerName, (long)LocationServicesUpdateFrequency, LocationServicesMinDistance, LocationListen);
					}
					catch (Exception e)
					{
						Log.debug("AndroidThunkJava_StartLocationService threw exception: " + e.getMessage());
						e.printStackTrace();
					}
				}
			});

			return true;
        }
        else
        {
			Log.debug("No Location Manager found!");
			return false;
        }
    }
	
	private String GetBestLocationProvider()
	{
		LocationManager locationManager = (LocationManager) activityContext.getSystemService(Context.LOCATION_SERVICE);

		Criteria providerCriteria = new Criteria();
		providerCriteria.setPowerRequirement(LocationServicesPower);
		providerCriteria.setHorizontalAccuracy(LocationServicesAccuracy);
		String providerName = locationManager.getBestProvider(providerCriteria, true);
		
		Log.debug("GetBestLocationProvider - getBestProvider for provided criteria returned: " + providerName);
		
		return providerName;
	}

    public boolean AndroidThunkJava_StopLocationService()
    {
		Log.debug("AndroidThunkJava_StopLocationService");

		if(LocationListen == null)
		{
			Log.debug("AndroidThunkJava_StopLocationService failed = no previous LocationListener registered, have not started the service");
			return false;
		}

		LocationManager locationManager = (LocationManager) this.getSystemService(Context.LOCATION_SERVICE);
		if(locationManager == null)
		{
			Log.debug("AndroidThunkJava_StopLocationService - No Location Manager found!");
			return false;
		}

		locationManager.removeUpdates(LocationListen);
		LocationListen = null;

		return true;
    }

    public float[] AndroidThunkJava_GetLastKnownLocation()
    {
		Location lastLocation = null;
		
        if(LastReportedLocation != null)
        {
			lastLocation = LastReportedLocation;
        }
        else
        {
			//if we haven't gotten an update about the location, try to fetch the LocationManager's last known location
			LocationManager locationManager = (LocationManager) this.getSystemService(Context.LOCATION_SERVICE);
			if(locationManager != null)
			{
				try
				{
					String providerName = GetBestLocationProvider();
					lastLocation = locationManager.getLastKnownLocation(providerName);
				}
				catch(Exception e) //in older APIs, isProviderEnabled will throw an exception if false...
				{
					Log.debug("AndroidThunkJava_GetLastKnownLocation threw exception: " + e.getMessage());
					e.printStackTrace();
				}
			}
        }
		
		if(lastLocation == null)
		{
			return new float[] { 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f };
		}
		else
		{
			return new float[]
			{
				(float)lastLocation.getTime(),
				(float)lastLocation.getLongitude(),
				(float)lastLocation.getLatitude(),
				lastLocation.getAccuracy(),
				0.0f, //no vertical accuracy on Android, just an overall value in horizontal
				(float)lastLocation.getAltitude()
			};
		}
    }
	
	public boolean AndroidThunkJava_IsLocationAccuracyAvailable(int Accuracy)
	{
		int locationAccurcy;
		int locationPower;
		
		LocationManager locationManager = (LocationManager)getSystemService(Context.LOCATION_SERVICE);
		if(locationManager == null)
		{
			Log.debug("AndroidThunkJava_IsLocationAccuracyAvailable - no LocationManager found!");
			return false;
		}
		
		switch(Accuracy)
		{
			case 0: //LA_ThreeKilometers: 
			{
				locationAccurcy = Criteria.ACCURACY_LOW;
				locationPower = Criteria.POWER_LOW;
				break;
			}
			case 1: //LA_OneKilometer: 
			{
				locationAccurcy = Criteria.ACCURACY_LOW;
				locationPower = Criteria.POWER_MEDIUM;
				break;
			}
			case 2: //LA_HundredMeters: 
			{
				locationAccurcy = Criteria.ACCURACY_MEDIUM;
				locationPower = Criteria.POWER_MEDIUM;
				break;
			}
			case 3: //LA_TenMeters: 
			{
				locationAccurcy = Criteria.ACCURACY_HIGH;
				locationPower = Criteria.POWER_MEDIUM;
				break;
			}
			case 4: //LA_Best: 
			case 5: //LA_Navigation:
			{
				locationAccurcy = Criteria.ACCURACY_HIGH;
				locationPower = Criteria.POWER_HIGH;
				break;
			}
			default: 
			{
				Log.debug("AndroidThunkJava_IsLocationAccuracyAvailable - calling with invalid Accuracy value:  " + Accuracy + "\n");
				return false;
			}
		}
		
		Criteria providerCriteria = new Criteria();
		providerCriteria.setPowerRequirement(locationPower);
		providerCriteria.setHorizontalAccuracy(locationAccurcy);
		String providerName = locationManager.getBestProvider(providerCriteria, true);
		
		return locationManager.isProviderEnabled(providerName);
	}

    public boolean AndroidThunkJava_IsLocationServiceEnabled()
    {
		try
		{
			LocationManager locationManager = (LocationManager)getSystemService(Context.LOCATION_SERVICE);
			if(locationManager != null)
			{
				//return true if either provider is available
				return locationManager.isProviderEnabled(LocationManager.NETWORK_PROVIDER) || locationManager.isProviderEnabled(LocationManager.GPS_PROVIDER);
			}
		}
		catch(Exception e) //in older APIs, isProviderEnabled will throw an exception if false...
		{
			Log.debug("AndroidThunkJava_IsLocationServiceEnabled threw exception: " + e.getMessage());
			e.printStackTrace();
		}

		return false;
    }

    public native void nativeHandleLocationChanged(long time, double longitude, double latitude, float accuracy, double altitude);

  </insert>
  </gameActivityClassAdditions>

</root>
