// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJniGameActivity.h"

namespace UE::Jni
{
	struct FLocationServices
	{
		using PartialClass = FGameActivity;

		static void JNICALL nativeHandleLocationChanged(JNIEnv* env, jobject thiz, jlong time, jdouble longitude, jdouble latitude, jfloat accuracy, jdouble altitude);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeHandleLocationChanged)
		};
	};
	
	template struct TInitialize<FLocationServices>;
}
#endif
