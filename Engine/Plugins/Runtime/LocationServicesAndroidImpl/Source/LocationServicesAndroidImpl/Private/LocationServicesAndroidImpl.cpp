// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesAndroidImpl.h"
#include "AndroidJniLocationServices.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "AndroidPermissionFunctionLibrary.h"
#include "Async/TaskGraphInterfaces.h"

DEFINE_LOG_CATEGORY(LogLocationServicesAndroid);

ULocationServicesAndroidImpl::~ULocationServicesAndroidImpl()
{

}

bool ULocationServicesAndroidImpl::InitLocationServices(ELocationAccuracy Accuracy, float UpdateFrequency, float MinDistanceFilter)
{
	TArray<FString> Permissions = { "android.permission.ACCESS_COARSE_LOCATION", "android.permission.ACCESS_FINE_LOCATION" };
	UAndroidPermissionFunctionLibrary::AcquirePermissions(Permissions);

	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID InitLocationServicesMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_InitLocationServices", "(IFF)Z", false);
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, InitLocationServicesMethod, (int)Accuracy, UpdateFrequency, MinDistanceFilter);
	}

	return false;
}

bool ULocationServicesAndroidImpl::StartLocationService()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID StartLocationServiceMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_StartLocationService", "()Z", false);
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, StartLocationServiceMethod);
	}

	return false;
}


bool ULocationServicesAndroidImpl::StopLocationService()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID StopLocationServiceMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_StopLocationService", "()Z", false);
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, StopLocationServiceMethod);
	}

	return false;
}

FLocationServicesData ULocationServicesAndroidImpl::GetLastKnownLocation()
{
	FLocationServicesData LocData;
	
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID GetLastKnownLocationMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_GetLastKnownLocation", "()[F", false);

		auto FloatValuesArray = NewScopedJavaObject(Env, (jfloatArray)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, GetLastKnownLocationMethod));

		jfloat* FloatValues = Env->GetFloatArrayElements(*FloatValuesArray, 0);

		if (Env->GetArrayLength(*FloatValuesArray) >= 6)
		{
			LocData.Timestamp = FloatValues[0];
			LocData.Longitude = FloatValues[1];
			LocData.Latitude = FloatValues[2];
			LocData.HorizontalAccuracy = FloatValues[3];
			LocData.VerticalAccuracy = FloatValues[4];
			LocData.Altitude = FloatValues[5];
		}

		Env->ReleaseFloatArrayElements(*FloatValuesArray, FloatValues, 0);
	}

	return LocData;
}

bool ULocationServicesAndroidImpl::IsLocationAccuracyAvailable(ELocationAccuracy Accuracy)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID IsLocationAccuracyAvailableMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_IsLocationAccuracyAvailable", "(I)Z", false);
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, IsLocationAccuracyAvailableMethod, (int)Accuracy);
	}

	return false;
}
	
bool ULocationServicesAndroidImpl::IsLocationServiceEnabled()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID IsLocationServiceEnabledMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_IsLocationServiceEnabled", "()Z", false);
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, IsLocationServiceEnabledMethod);
	}

	return false;
}


void JNICALL UE::Jni::FLocationServices::nativeHandleLocationChanged(JNIEnv* jenv, jobject thiz, jlong time, jdouble longitude, jdouble latitude, jfloat accuracy, jdouble altitude)
{
	//we're passing this value up to Blueprints, which only takes floats.
	FLocationServicesData LocationData;
	LocationData.Timestamp = (float)time;
	LocationData.Longitude = (float)longitude;
	LocationData.Latitude = (float)latitude;
	LocationData.HorizontalAccuracy = accuracy;
	LocationData.VerticalAccuracy = 0.0f;
	LocationData.Altitude = (float)altitude;

	if (FTaskGraphInterface::IsRunning())
	{
		FGraphEventRef BroadcastLocationTask = FFunctionGraphTask::CreateAndDispatchWhenReady([LocationData]()
		{
			ULocationServices::GetLocationServicesImpl()->OnLocationChanged.Broadcast(LocationData);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}
