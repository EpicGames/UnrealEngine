// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocationServicesAndroidImplModule.h"
#include "LocationServicesAndroidImpl.h"
#include "LocationServicesBPLibrary.h"

IMPLEMENT_MODULE(FLocationServicesAndroidImplModule, LocationServicesAndroidImpl)

void FLocationServicesAndroidImplModule::StartupModule()
{
	ImplInstance = NewObject<ULocationServicesAndroidImpl>();
	ULocationServices::SetLocationServicesImpl(ImplInstance);
}

void FLocationServicesAndroidImplModule::ShutdownModule()
{
	ULocationServices::ClearLocationServicesImpl();
	ImplInstance = NULL;
}