// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class ULocationServicesAndroidImpl;
/**
 * The public interface to this module
 */
class FLocationServicesAndroidImplModule :
	public IModuleInterface
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FLocationServicesAndroidImplModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FLocationServicesAndroidImplModule >( "LocationServicesAndroidImpl" );
	}

private:
	ULocationServicesAndroidImpl* ImplInstance;
	
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

